{
	"FileVersion": 3,
	"Version": 2,
	"VersionName": "0.2",
	"FriendlyName": "PCG Biome Sample",
	"Description": "PCG Biome Sample Plugin",
	"Category": "Other",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Plugins": [
		{
			"Name": "PCGBiomeCore",
			"Enabled": true
		},
		{
			"Name": "PCG",
			"Enabled": true
		}
	]
}